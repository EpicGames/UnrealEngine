{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Android Runtime Permission",
	"Description": "Support for Android Runtime Permission",
	"Category": "Android",
	"CreatedBy": "Google",
	"CreatedByURL": "https://developers.google.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AndroidPermission",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}