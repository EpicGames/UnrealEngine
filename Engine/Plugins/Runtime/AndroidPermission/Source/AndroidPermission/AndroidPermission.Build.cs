// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class AndroidPermission : ModuleRules
{
	public AndroidPermission(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"Engine",
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Slate",
				"SlateCore",
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			}
			);
        if (Target.Platform == UnrealTargetPlatform.Android)
        {
            string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
            AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "AndroidPermission_APL.xml"));
        }
    }
}
