<?xml version="1.0" encoding="utf-8"?>
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<init>
		<log text="Android Permission Plugin Init"/>
	</init>
	
	<prebuildCopies>
		<log text="Copying permission_library directory to staging before build"/>
		<copyDir src="$S(EngineDir)/Source/ThirdParty/AndroidPermission/permission_library"
					dst="$S(BuildDir)/JavaLibs/permission_library" />
	</prebuildCopies>

	<proguardAdditions>
		<insert>
			-keep class com.google.vr.sdk.samples.permission.** {
				*;
			}
		</insert>
	</proguardAdditions>

	<buildGradleAdditions>
		<insert>
dependencies {
	implementation('com.android.support:support-v13:27.1.0')
}
		</insert>
	</buildGradleAdditions>
</root>


