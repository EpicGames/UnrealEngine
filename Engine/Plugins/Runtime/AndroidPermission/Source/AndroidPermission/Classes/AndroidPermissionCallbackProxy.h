// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AndroidPermissionCallbackProxy.generated.h"

#define UE_API ANDROIDPERMISSION_API

DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FAndroidPermissionDynamicDelegate, const TArray<FString>&, Permissions, const TArray<bool>&, GrantResults);
DECLARE_MULTICAST_DELEGATE_TwoParams(FAndroidPermissionDelegate, const TArray<FString>& /*Permissions*/, const TArray<bool>& /*GrantResults*/);

UCLASS(MinimalAPI)
class UAndroidPermissionCallbackProxy : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintAssignable, Category="AndroidPermission")
	FAndroidPermissionDynamicDelegate OnPermissionsGrantedDynamicDelegate;

	FAndroidPermissionDelegate OnPermissionsGrantedDelegate;
	
	static UE_API UAndroidPermissionCallbackProxy *GetInstance();
};

#undef UE_API
