// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "AndroidPermissionFunctionLibrary.generated.h"

#define UE_API ANDROIDPERMISSION_API

class UAndroidPermissionCallbackProxy;

UCLASS(MinimalAPI)
class UAndroidPermissionFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/** check if the permission is already granted */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Check Android Permission"), Category="AndroidPermission")
	static UE_API bool CheckPermission(const FString& permission);

	/** try to acquire permissions and return a singleton callback proxy object containing OnPermissionsGranted delegate */
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Request Android Permissions"), Category="AndroidPermission")
	static UE_API UAndroidPermissionCallbackProxy* AcquirePermissions(const TArray<FString>& permissions);

public:
	/** initialize java objects and cache them for further usage. called when the module is loaded */
	static UE_API void Initialize();
};

#undef UE_API
