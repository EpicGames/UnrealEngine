// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidPermission.h"
#include "AndroidPermissionFunctionLibrary.h"
#include "AndroidPermissionCallbackProxy.h"
#include "Modules/ModuleManager.h"
#if PLATFORM_ANDROID
#include "Android/AndroidApplication.h"
#endif

#define LOCTEXT_NAMESPACE "FAndroidPermissionModule"

void FAndroidPermissionModule::StartupModule()
{
	// prepare java class and method cache
	UAndroidPermissionFunctionLibrary::Initialize();
	// prepare the singleton proxy object
	UAndroidPermissionCallbackProxy::GetInstance();
}

void FAndroidPermissionModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAndroidPermissionModule, AndroidPermission)
