// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidPermissionCallbackProxy.h"
#include "AndroidPermission.h"

#include "Trace/Trace.inl"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AndroidPermissionCallbackProxy)

#if PLATFORM_ANDROID
#include "Android/AndroidApplication.h"
#include "Async/TaskGraphInterfaces.h"
#endif

static UAndroidPermissionCallbackProxy *pProxy = NULL;

UAndroidPermissionCallbackProxy *UAndroidPermissionCallbackProxy::GetInstance()
{
	if (!pProxy) {
		pProxy = NewObject<UAndroidPermissionCallbackProxy>();
		pProxy->AddToRoot();

	}
	UE_LOGF(LogAndroidPermission, Log, "UAndroidPermissionCallbackProxy::GetInstance");
	return pProxy;
}

#if PLATFORM_ANDROID && USE_ANDROID_JNI
JNI_METHOD void Java_com_google_vr_sdk_samples_permission_PermissionHelper_onAcquirePermissions(JNIEnv *env, jclass clazz, jobjectArray permissions, jintArray grantResults) 
{
	if (!pProxy) return;

	TArray<FString> arrPermissions;
	TArray<bool> arrGranted;
	int num = env->GetArrayLength(permissions);
	jint* jarrGranted = env->GetIntArrayElements(grantResults, 0);
	for (int i = 0; i < num; i++)
	{
		arrPermissions.Add(FJavaHelper::FStringFromLocalRef(env, (jstring)env->GetObjectArrayElement(permissions, i)));
		arrGranted.Add(jarrGranted[i] == 0 ? true : false); // 0: permission granted, -1: permission denied
	}
	env->ReleaseIntArrayElements(grantResults, jarrGranted, 0);

	UE_LOGF(LogAndroidPermission, Log, "PermissionHelper_onAcquirePermissions %ls %d (%d), Broadcasting...",
		*(arrPermissions[0]), arrGranted[0], num);

	if (FTaskGraphInterface::IsRunning())
	{
		FGraphEventRef AquirePermissionsTask = FFunctionGraphTask::CreateAndDispatchWhenReady([arrPermissions, arrGranted]()
		{
			// pProxy has to be valid if got here (never reset after set)
			pProxy->OnPermissionsGrantedDelegate.Broadcast(arrPermissions, arrGranted);
			pProxy->OnPermissionsGrantedDynamicDelegate.Broadcast(arrPermissions, arrGranted);
		}, TStatId(), NULL, ENamedThreads::GameThread);
	}
}
#endif

