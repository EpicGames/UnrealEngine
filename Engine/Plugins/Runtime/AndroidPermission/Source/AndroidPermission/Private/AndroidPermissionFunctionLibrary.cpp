// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidPermissionFunctionLibrary.h"
#include "AndroidPermission.h"
#include "AndroidPermissionCallbackProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AndroidPermissionFunctionLibrary)

#if PLATFORM_ANDROID
#include "Android/AndroidApplication.h"

#if USE_ANDROID_JNI
#include "Android/AndroidJNI.h"
static jclass _PermissionHelperClass;
static jmethodID _CheckPermissionMethodId;
static jmethodID _AcquirePermissionMethodId;
#endif
#endif

DEFINE_LOG_CATEGORY(LogAndroidPermission);

void UAndroidPermissionFunctionLibrary::Initialize()
{
#if PLATFORM_ANDROID && USE_ANDROID_JNI
	JNIEnv* env = FAndroidApplication::GetJavaEnv();
	_PermissionHelperClass = FAndroidApplication::FindJavaClassGlobalRef("com/google/vr/sdk/samples/permission/PermissionHelper");
	_CheckPermissionMethodId = env->GetStaticMethodID(_PermissionHelperClass, "checkPermission", "(Ljava/lang/String;)Z");
	_AcquirePermissionMethodId = env->GetStaticMethodID(_PermissionHelperClass, "acquirePermissions", "([Ljava/lang/String;)V");
#endif
}

bool UAndroidPermissionFunctionLibrary::CheckPermission(const FString& permission)
{
#if PLATFORM_ANDROID && USE_ANDROID_JNI
	UE_LOGF(LogAndroidPermission, Log, "UAndroidPermissionFunctionLibrary::CheckPermission %ls (Android)", *permission);
	JNIEnv* env = FAndroidApplication::GetJavaEnv();
	auto argument = FJavaHelper::ToJavaString(env, permission);
	bool bResult = env->CallStaticBooleanMethod(_PermissionHelperClass, _CheckPermissionMethodId, *argument);
	return bResult;
#else
	UE_LOGF(LogAndroidPermission, Log, "UAndroidPermissionFunctionLibrary::CheckPermission (Else)");
	return false;
#endif
}

UAndroidPermissionCallbackProxy* UAndroidPermissionFunctionLibrary::AcquirePermissions(const TArray<FString>& permissions)
{
#if PLATFORM_ANDROID && USE_ANDROID_JNI
	UE_LOGF(LogAndroidPermission, Log, "UAndroidPermissionFunctionLibrary::AcquirePermissions");
	JNIEnv* env = FAndroidApplication::GetJavaEnv();
	auto permissionsArray = NewScopedJavaObject(env, (jobjectArray)env->NewObjectArray(permissions.Num(), FJavaWrapper::JavaStringClass, NULL));
	for (int i = 0; i < permissions.Num(); i++)
	{
		auto str = FJavaHelper::ToJavaString(env, permissions[i]);
		env->SetObjectArrayElement(*permissionsArray, i, *str);
	}
	env->CallStaticVoidMethod(_PermissionHelperClass, _AcquirePermissionMethodId, *permissionsArray);
	return UAndroidPermissionCallbackProxy::GetInstance();
#else
	UE_LOGF(LogAndroidPermission, Log, "UAndroidPermissionFunctionLibrary::AcquirePermissions(%ls) (Android)", *(FString::Join(permissions, TEXT(","))));
	return UAndroidPermissionCallbackProxy::GetInstance();
#endif
}

