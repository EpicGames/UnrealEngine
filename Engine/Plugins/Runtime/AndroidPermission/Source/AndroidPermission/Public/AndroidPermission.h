// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

ANDROIDPERMISSION_API DECLARE_LOG_CATEGORY_EXTERN(LogAndroidPermission, Log, All);

class FAndroidPermissionModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
